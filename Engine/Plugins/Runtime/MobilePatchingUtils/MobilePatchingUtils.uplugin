{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Mobile Patching Utilities",
	"Description" : "Blueprint exposed functionality for downloading and patching content on mobile platforms",
	"Category" : "Mobile",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "MobilePatchingUtils",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		}
	]
}