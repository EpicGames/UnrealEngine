// Copyright Epic Games, Inc. All Rights Reserved.

#include "MobilePatchingLibrary.h"
#include "BuildPatchState.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Interfaces/IHttpRequest.h"
#include "Modules/ModuleManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "DeviceProfiles/DeviceProfileManager.h"

#include "Interfaces/IBuildPatchServicesModule.h"
#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "IPlatformFilePak.h"
#include "Math/UnitConversion.h"
#include "Misc/ScopeExit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MobilePatchingLibrary)

#define LOCTEXT_NAMESPACE "MobilePatchingUtils"

DEFINE_LOG_CATEGORY_STATIC(LogMobilePatchingUtils, Log, All);

static FString GetStagingDir()
{
	return FPaths::ProjectPersistentDownloadDir() / TEXT("PatchStaging");
}

static IBuildPatchServicesModule* GetBuildPatchServices()
{
	static IBuildPatchServicesModule* BuildPatchServices = nullptr;
	if (BuildPatchServices == nullptr)
	{
		BuildPatchServices = &FModuleManager::LoadModuleChecked<IBuildPatchServicesModule>(TEXT("BuildPatchServices"));
	}
	return BuildPatchServices;
}

static const FText& GetBuildPatchStateText(const BuildPatchServices::EBuildPatchState& State)
{
	static const FText Queued = LOCTEXT("EBuildPatchState_Queued", "Queued");
	static const FText Initializing = LOCTEXT("EBuildPatchState_Initialising", "Initializing");
	static const FText Resuming = LOCTEXT("EBuildPatchState_Resuming", "Resuming");
	static const FText Downloading = LOCTEXT("EBuildPatchState_Downloading", "Downloading");
	static const FText Installing = LOCTEXT("EBuildPatchState_Installing", "Installing");
	static const FText BuildVerification = LOCTEXT("EBuildPatchState_BuildVerification", "Verifying");
	static const FText CleanUp = LOCTEXT("EBuildPatchState_CleanUp", "Cleaning up");
	static const FText PrerequisitesInstall = LOCTEXT("EBuildPatchState_PrerequisitesInstall", "Prerequisites");
	static const FText Completed = LOCTEXT("EBuildPatchState_Complete", "Complete");
	static const FText Paused = LOCTEXT("EBuildPatchState_Paused", "Paused");
	static const FText Empty = FText::GetEmpty();

	switch (State)
	{
		case BuildPatchServices::EBuildPatchState::Queued:
			return Queued;
		case BuildPatchServices::EBuildPatchState::Initializing:
			return Initializing;
		case BuildPatchServices::EBuildPatchState::Resuming:
			return Resuming;
		case BuildPatchServices::EBuildPatchState::Downloading:
			return Downloading;
		case BuildPatchServices::EBuildPatchState::Installing:
			// falls through
		case BuildPatchServices::EBuildPatchState::MovingToInstall:
			// falls through
		case BuildPatchServices::EBuildPatchState::SettingAttributes:
			return Installing;
		case BuildPatchServices::EBuildPatchState::BuildVerification:
			return BuildVerification;
		case BuildPatchServices::EBuildPatchState::CleanUp:
			return CleanUp;
		case BuildPatchServices::EBuildPatchState::PrerequisitesInstall:
			return PrerequisitesInstall;
		case BuildPatchServices::EBuildPatchState::Completed:
			return Completed;
		case BuildPatchServices::EBuildPatchState::Paused:
			return Paused;
		default:
			return Empty;
	}
}

static const FText& GetRequestContentErrorText(ERequestContentError ErrorCode)
{
	static const FText NoError(LOCTEXT("RequestContentError_NoError", "The operation was successful."));
	static const FText InvalidInstallationDirectory(LOCTEXT("RequestContentError_InvalidInstallationDirectory", "Invalid installation directory"));
	static const FText InvalidCloudURL(LOCTEXT("RequestContentError_InvalidCloudURL", "Invalid cloud URL"));
	static const FText InvalidManifestURL(LOCTEXT("RequestContentError_InvalidManifestURL", "Invalid manifest URL"));
	static const FText FailedToDownloadManifestNoResponse(LOCTEXT("RequestContentError_FailedToDownloadManifestNoResponse", "Failed to download manifest file. No response"));
	static const FText FailedToDownloadManifest(LOCTEXT("RequestContentError_FailedToDownloadManifest", "Failed to download manifest file"));
	static const FText FailedToReadManifest(LOCTEXT("RequestContentError_FailedToReadManifest", "Failed to reconstruct downloaded manifest file"));
	static const FText InvalidOrMax(LOCTEXT("RequestContentError_InvalidOrMax", "An unknown error ocurred"));

	switch (ErrorCode)
	{
	case ERequestContentError::NoError: return NoError;
	case ERequestContentError::InvalidInstallationDirectory: return InvalidInstallationDirectory;
	case ERequestContentError::InvalidCloudURL: return InvalidCloudURL;
	case ERequestContentError::InvalidManifestURL: return InvalidManifestURL;
	case ERequestContentError::FailedToDownloadManifestNoResponse: return FailedToDownloadManifestNoResponse;
	case ERequestContentError::FailedToDownloadManifest: return FailedToDownloadManifest;
	case ERequestContentError::FailedToReadManifest: return FailedToReadManifest;
	default: return InvalidOrMax;
	}
}

float UMobileInstalledContent::GetInstalledContentSize()
{
	const double BuildSize = InstalledManifest.IsValid() ? (double)InstalledManifest->GetBuildSize() : 0.0;
	const double BuildSizeMB = FUnitConversion::Convert(BuildSize, EUnit::Bytes, EUnit::Megabytes);
	return (float)BuildSizeMB;
}

float UMobileInstalledContent::GetDiskFreeSpace()
{
	uint64 TotalDiskSpace = 0;
	uint64 TotalDiskFreeSpace = 0;
	if (FPlatformMisc::GetDiskTotalAndFreeSpace(InstallDir, TotalDiskSpace, TotalDiskFreeSpace))
	{
		double DiskFreeSpace = (double)TotalDiskFreeSpace;
		double DiskFreeSpaceMB = FUnitConversion::Convert(DiskFreeSpace, EUnit::Bytes, EUnit::Megabytes);
		return (float)DiskFreeSpaceMB;
	}
	return 0.0f;
}

bool UMobileInstalledContent::Mount(int32 InPakOrder, const FString& InMountPoint)
{
	// Mount all pak files found in this content
	FPakPlatformFile* PakFileMgr = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(TEXT("PakFile")));
	if (PakFileMgr == nullptr)
	{
		return false;
	}

	bool bMounted = false;
	uint32 PakOrder = (uint32)FMath::Max(0, InPakOrder);
	const TCHAR* MountPount = InMountPoint.GetCharArray().GetData();

	if (InstalledManifest.IsValid())
	{
		TArray<FString> InstalledFileNames = InstalledManifest->GetBuildFileList();
		for (const FString& FileName : InstalledFileNames)
		{
			if (FPaths::GetExtension(FileName) == TEXT("pak"))
			{
				FString PakFullName = InstallDir / FileName;
				if (PakFileMgr->Mount(*PakFullName, PakOrder, MountPount))
				{
					UE_LOGF(LogMobilePatchingUtils, Log, "Mounted = %ls, Order = %d, MountPoint = %ls", *PakFullName, PakOrder, !MountPount ? TEXT("(null)") : MountPount);
					bMounted = true;
				}
				else
				{
					UE_LOGF(LogMobilePatchingUtils, Error, "Failed to mount pak = %ls", *PakFullName);
					bMounted = false;
					break;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogMobilePatchingUtils, Log, "No installed manifest, failed to mount");
	}

	return bMounted;
}

float UMobilePendingContent::GetDownloadSize()
{
	double DonwloadSize = 0.0;

	if (RemoteManifest.IsValid())
	{
		if (InstalledManifest.IsValid())
		{
			TSet<FString> Tags;
			DonwloadSize = (double)RemoteManifest->GetDeltaDownloadSize(Tags, InstalledManifest.ToSharedRef());
		}
		else
		{
			DonwloadSize = (double)RemoteManifest->GetDownloadSize();
		}
	}
	
	double DonwloadSizeMB = FUnitConversion::Convert(DonwloadSize, EUnit::Bytes, EUnit::Megabytes);	
	return (float)DonwloadSizeMB;
}

float UMobilePendingContent::GetRequiredDiskSpace()
{
	if (RemoteManifest.IsValid())
	{
		double BuildSize = (double)RemoteManifest->GetBuildSize();
		double BuildSizeMB = FUnitConversion::Convert(BuildSize, EUnit::Bytes, EUnit::Megabytes);
		return (float)BuildSizeMB;
	}

	return 0.0f;
}

float UMobilePendingContent::GetDownloadSpeed()
{
	if (Installer.IsValid())
	{
		double SpeedBytes = (double)Installer->GetDownloadSpeed();
		double SpeedMB = FUnitConversion::Convert(SpeedBytes, EUnit::Bytes, EUnit::Megabytes);
		return (float)SpeedMB;
	}
	return 0.0f;
}

float UMobilePendingContent::GetTotalDownloadedSize()
{
	if (Installer.IsValid())
	{
		double SizeBytes = (double)Installer->GetTotalDownloaded();
		double SizeMB = FUnitConversion::Convert(SizeBytes, EUnit::Bytes, EUnit::Megabytes);
		return (float)SizeMB;
	}
	return 0.0f;
}

FText UMobilePendingContent::GetDownloadStatusText()
{
	// DEPRECATED 4.21
	
	//if (Installer.IsValid())
	//{
	//	return GetBuildPatchStateText(Installer->GetState());
	//}
	return FText::GetEmpty();
}

float UMobilePendingContent::GetInstallProgress()
{
	if (Installer.IsValid())
	{
		return Installer->GetUpdateProgress();
	}
	return 0.0f;
}

static void OnInstallComplete(const IBuildInstallerRef& Installer, UMobilePendingContent* MobilePendingContent, FOnContentInstallSucceeded OnSucceeded, FOnContentInstallFailed OnFailed)
{
	if (MobilePendingContent == nullptr)
	{
		// Don't do anything if owner is gone
		UE_LOGF(LogMobilePatchingUtils, Error, "Installation Failed. MobilePendingContent is null");
		return;
	}
	
	if (Installer->CompletedSuccessfully())
	{
		IBuildPatchServicesModule* BuildPatchServices = GetBuildPatchServices();
		FString ManifestFilename = FPaths::GetCleanFilename(MobilePendingContent->RemoteManifestURL);
		FString ManifestFullFilename = MobilePendingContent->InstallDir / ManifestFilename;
		
		if (!BuildPatchServices->SaveManifestToFile(ManifestFullFilename, MobilePendingContent->RemoteManifest.ToSharedRef()))
		{
			UE_LOGF(LogMobilePatchingUtils, Error, "Failed to save manifest to disk. %ls", *ManifestFullFilename);
		}

		// Installed content updated
		MobilePendingContent->InstalledManifest = MobilePendingContent->RemoteManifest;
		OnSucceeded.ExecuteIfBound();
	}
	else
	{
		EBuildPatchInstallError ErrorCode = EBuildPatchInstallError::NumInstallErrors;
		FText ErrorText = LOCTEXT("Error_Unknown", "An unknown error ocurred");
		
		if (MobilePendingContent->Installer.IsValid())
		{
			ErrorText = MobilePendingContent->Installer->GetErrorText();
			ErrorCode = MobilePendingContent->Installer->GetErrorType();
		}
		
		UE_LOGF(LogMobilePatchingUtils, Error, "Installation Failed. Code %d Msg: %ls", (int32)ErrorCode, *ErrorText.ToString());
		OnFailed.ExecuteIfBound(ErrorText, (int32)ErrorCode);
	}

	// Release installer
	MobilePendingContent->Installer.Reset();
}

void UMobilePendingContent::StartInstall(FOnContentInstallSucceeded OnSucceeded, FOnContentInstallFailed OnFailed)
{
	const FString StageDir = GetStagingDir();
	IBuildPatchServicesModule* BuildPatchServices = GetBuildPatchServices();
	
	float DownloadSize = GetDownloadSize();
	float RequiredDiskSpace = GetRequiredDiskSpace();
	float DiskFreeSpace = GetDiskFreeSpace();
	UE_LOGF(LogMobilePatchingUtils, Log, "Download size = %.2f MB", DownloadSize);
	UE_LOGF(LogMobilePatchingUtils, Log, "Required disk space = %.2f MB", RequiredDiskSpace);
	UE_LOGF(LogMobilePatchingUtils, Log, "Disk free space = %.2f MB", DiskFreeSpace);

	TArray<BuildPatchServices::FInstallerAction> InstallerActions;
	InstallerActions.Add(BuildPatchServices::FInstallerAction::MakeInstallOrUpdate(InstalledManifest, RemoteManifest.ToSharedRef(), TSet<FString>()));

	BuildPatchServices::FBuildInstallerConfiguration Configuration(MoveTemp(InstallerActions));
	Configuration.StagingDirectory = StageDir;
	Configuration.CloudDirectories = { CloudURL };
	Configuration.InstallDirectory = InstallDir;

	Installer = BuildPatchServices->CreateBuildInstaller(Configuration, FBuildPatchInstallerDelegate::CreateStatic(OnInstallComplete, this, OnSucceeded, OnFailed));
	Installer->StartInstallation();
}

void UMobilePendingContent::BeginDestroy()
{
	Super::BeginDestroy();
	
	if (Installer.IsValid())
	{
		Installer->CancelInstall();
		Installer.Reset();
	}
}

static void OnRemoteManifestReady(FHttpRequestPtr Request, FHttpResponsePtr Response, bool bSucceeded, UMobilePendingContent* MobilePendingContent, FOnRequestContentSucceeded OnSucceeded, FOnRequestContentFailed OnFailed)
{
	ERequestContentError ErrorCode = ERequestContentError::NoError;
	ON_SCOPE_EXIT
	{
		if (ErrorCode != ERequestContentError::NoError)
		{
			FText ErrorText = GetRequestContentErrorText(ErrorCode);
			UE_LOGF(LogMobilePatchingUtils, Error, "ErrorText: %ls. Code %d", *ErrorText.ToString(), (int32)ErrorCode);
			OnFailed.ExecuteIfBound(ErrorText, (int32)ErrorCode);
		}
	};

	if (MobilePendingContent)
	{
		// We don't need to keep this object alive anymore, 
		// on Success it will be passed to the User and it's up to them to keep a reference to it
		MobilePendingContent->RemoveFromRoot();
	}
		
	if (!bSucceeded || !Response.IsValid())
	{
		ErrorCode = ERequestContentError::FailedToDownloadManifestNoResponse;
		return;
	}

	int32 ResponseCode = Response->GetResponseCode();
	if (!EHttpResponseCodes::IsOk(ResponseCode))
	{
		UE_LOGF(LogMobilePatchingUtils, Error, "Failed to download manifest. ResponseCode = %d, ResponseMsg = %ls", ResponseCode, *Response->GetContentAsString());
		ErrorCode = ERequestContentError::FailedToDownloadManifest;
		return;
	}
	
	IBuildPatchServicesModule* BuildPatchServices = GetBuildPatchServices();
	IBuildManifestPtr RemoteManifest = BuildPatchServices->MakeManifestFromData(Response->GetContent());
	if (!RemoteManifest.IsValid())
	{
		ErrorCode = ERequestContentError::FailedToReadManifest;
		return;
	}

	if (MobilePendingContent)
	{
		MobilePendingContent->RemoteManifest = RemoteManifest;
		OnSucceeded.ExecuteIfBound(MobilePendingContent);
	}
}

static IBuildManifestPtr GetInstalledManifest(const FString& InstallDirectory)
{
	IBuildManifestPtr InstalledManifest;
	
	FString FullInstallDir = FPaths::ProjectPersistentDownloadDir() / InstallDirectory;
	TArray<FString> InstalledManifestNames;
	IFileManager::Get().FindFiles(InstalledManifestNames, *(FullInstallDir / TEXT("*.manifest")), true, false);

	if (InstalledManifestNames.Num()) 
	{
		const FString& InstalledManifestName = InstalledManifestNames[0];// should we warn if there more than one manifest?
		IBuildPatchServicesModule* BuildPatchServices = GetBuildPatchServices();
		InstalledManifest = BuildPatchServices->LoadManifestFromFile(FullInstallDir / InstalledManifestName);
	}

	return InstalledManifest;
}

UMobileInstalledContent* UMobilePatchingLibrary::GetInstalledContent(const FString& InstallDirectory)
{
	UMobileInstalledContent* InstalledContent = nullptr;
	// Look for installed manifest
	IBuildManifestPtr InstalledManifest = GetInstalledManifest(InstallDirectory);
	if (InstalledManifest.IsValid())
	{
		InstalledContent = NewObject<UMobileInstalledContent>();
		InstalledContent->InstallDir = FPaths::ProjectPersistentDownloadDir() / InstallDirectory;
		InstalledContent->InstalledManifest = InstalledManifest;
	}
	
	return InstalledContent;
}

void UMobilePatchingLibrary::RequestContent(const FString& RemoteManifestURL, const FString& CloudURL, const FString& InstallDirectory, FOnRequestContentSucceeded OnSucceeded, FOnRequestContentFailed OnFailed)
{
	ERequestContentError ErrorCode = ERequestContentError::NoError;
	ON_SCOPE_EXIT
	{
		if (ErrorCode != ERequestContentError::NoError)
		{
			FText ErrorText = GetRequestContentErrorText(ErrorCode);
			UE_LOGF(LogMobilePatchingUtils, Error, "ErrorText: %ls. Code %d", *ErrorText.ToString(), (int32)ErrorCode);
			OnFailed.ExecuteIfBound(ErrorText, (int32)ErrorCode);
		}
	};
		
	if (InstallDirectory.IsEmpty())
	{
		ErrorCode = ERequestContentError::InvalidInstallationDirectory;
		return;
	}

	if (RemoteManifestURL.IsEmpty())
	{
		ErrorCode = ERequestContentError::InvalidManifestURL;
		return;
	}

	if (CloudURL.IsEmpty())
	{
		ErrorCode = ERequestContentError::InvalidCloudURL;
		return;
	}
		
	UMobilePendingContent* MobilePendingContent = NewObject<UMobilePendingContent>();
	MobilePendingContent->InstallDir = FPaths::ProjectPersistentDownloadDir() / InstallDirectory;
	MobilePendingContent->RemoteManifestURL = RemoteManifestURL;
	MobilePendingContent->CloudURL = CloudURL;
	MobilePendingContent->InstalledManifest = GetInstalledManifest(InstallDirectory);
	// We need to keep this object alive until we pass it to the User
	MobilePendingContent->AddToRoot();

	// Request remote manifest
	TSharedRef<class IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->OnProcessRequestComplete().BindStatic(OnRemoteManifestReady, MobilePendingContent, OnSucceeded, OnFailed);
	HttpRequest->SetURL(*RemoteManifestURL);
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->ProcessRequest();
}

bool UMobilePatchingLibrary::HasActiveWiFiConnection()
{
	return FPlatformMisc::HasActiveWiFiConnection();
}

FString UMobilePatchingLibrary::GetActiveDeviceProfileName()
{
	UDeviceProfile* DeviceProfile = UDeviceProfileManager::Get().GetActiveProfile();
	check(DeviceProfile);
	return DeviceProfile->GetName();
}

TArray<FString> UMobilePatchingLibrary::GetSupportedPlatformNames()
{
	TArray<FString> Result;
	FPlatformMisc::GetValidTargetPlatforms(Result);
	return Result;
}

#undef LOCTEXT_NAMESPACE
