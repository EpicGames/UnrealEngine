// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildManifest.h"
#include "MobilePatchingLibrary.generated.h"

#define UE_API MOBILEPATCHINGUTILS_API

DECLARE_DYNAMIC_DELEGATE(FOnContentInstallSucceeded);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnContentInstallFailed, FText, ErrorText, int32, ErrorCode);

UCLASS(MinimalAPI, BlueprintType)
class UMobileInstalledContent : public UObject
{
	GENERATED_BODY()

public:
	/** Get the disk free space in megabytes where content is installed  */
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	UE_API float GetDiskFreeSpace();

	/** Get the installed content size in megabytes */
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	UE_API float GetInstalledContentSize();

	/** 
	 * Mount installed content
	 * @param	PakOrder : Content pak priority
	 * @param	MountPoint : Path to mount the pak at
	 */
	UFUNCTION(BlueprintCallable, Category="Mobile Patching", meta=(AdvancedDisplay="PakOrder,MountPoint", PakOrder="1"))
	UE_API bool Mount(int32 PakOrder, const FString& MountPoint);

public:	
	// User specified directory where content should be/already installed
	FString InstallDir;
	// Currently installed manifest
	IBuildManifestPtr InstalledManifest;
};

UCLASS(MinimalAPI, BlueprintType)
class UMobilePendingContent : public UMobileInstalledContent
{
	GENERATED_BODY()

public:	
	/** Get the total download size for this content installation */
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	UE_API float GetDownloadSize();
	
	/** Get the required disk space in megabytes for this content installation */
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	UE_API float GetRequiredDiskSpace();

	/** Get the total downloaded size in megabytes. Valid during installation */
	UFUNCTION(BlueprintPure, Category="Mobile Patching|Progress")
	UE_API float GetTotalDownloadedSize();

	/** Get the current download speed in megabytes per second. Valid during installation */
	UFUNCTION(BlueprintPure, Category="Mobile Patching|Progress")
	UE_API float GetDownloadSpeed();

	/** DEPRECATED. GetDownloadStatusText has been deprecated. It will no longer be supported in the future */
	UE_DEPRECATED(4.21, "GetDownloadStatusText has been deprecated.  It will no longer be supported in the future.")
	UFUNCTION(BlueprintPure, Category="Mobile Patching|Progress")
	UE_API FText GetDownloadStatusText();
	
	/** Get the current installation progress. Between 0 and 1 for known progress, or less than 0 for unknown progress */
	UFUNCTION(BlueprintPure, Category="Mobile Patching|Progress")
	UE_API float GetInstallProgress();
	
	 /** 
	 * Attempt to download and install remote content.
	 * User can choose to mount installed content into the game.
	 * @param	OnSucceeded: Callback on installation success. 
	 * @param	OnFailed: Callback on installation fail. Will return error message text and error integer code
	 */
	UFUNCTION(BlueprintCallable, Category="Mobile Patching")
	UE_API void StartInstall(FOnContentInstallSucceeded OnSucceeded, FOnContentInstallFailed OnFailed);

	//~ Begin UObject Interface
	UE_API virtual void BeginDestroy() override;
	//~ End UObject Interface.

public:
	// User specified URL from where manifest can be downloaded
	FString RemoteManifestURL;
	// User specified cloud URL from where content chunks can be downloaded
	FString CloudURL;
	// Content installer, only valid during installation
	IBuildInstallerPtr Installer;
	// Manifest downloaded from a cloud
	IBuildManifestPtr RemoteManifest;
};

enum class ERequestContentError
{
	NoError,
	InvalidInstallationDirectory,
	InvalidCloudURL,
	InvalidManifestURL,
	FailedToDownloadManifestNoResponse,
	FailedToDownloadManifest,
	FailedToReadManifest,
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnRequestContentSucceeded, UMobilePendingContent*, MobilePendingContent);
DECLARE_DYNAMIC_DELEGATE_TwoParams(FOnRequestContentFailed, FText, ErrorText, int32, ErrorCode);

UCLASS(MinimalAPI)
class UMobilePatchingLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Get the installed content. Will return non-null object if there is an installed content at specified directory
	 * User can choose to mount installed content into the game
	 */
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	static UE_API UMobileInstalledContent* GetInstalledContent(const FString& InstallDirectory);
	
	/** 
	 * Attempt to download manifest file using specified manifest URL. 
	 * On success it will return an object that represents remote content. This object can be queried for additional information, like total content size, download size, etc.
	 * User can choose to download and install remote content.
	 * @param	RemoteManifestURL : URL from where manifest file can be downloaded. (http://my-server.com/awesomecontent.manifest)
	 * @param	CloudURL :  URL from where content chunk data can be downloaded. (http://my-server.com/awesomecontent/clouddir)
	 * @param	InstallDirectory: Relative directory to where downloaded content should be installed (MyContent/AwesomeContent)
	 * @param	OnSucceeded: Callback on manifest download success. Will return object that represents remote content 
	 * @param	OnFailed: Callback on manifest download fail. Will return error message text and error integer code
	 */
	UFUNCTION(BlueprintCallable, Category="Mobile Patching")
	static UE_API void RequestContent(const FString& RemoteManifestURL, const FString& CloudURL, const FString& InstallDirectory, FOnRequestContentSucceeded OnSucceeded, FOnRequestContentFailed OnFailed);

	// Whether WiFi connection is currently available
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	static UE_API bool HasActiveWiFiConnection();

	// Get the name of currently selected device profile name
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	static UE_API FString GetActiveDeviceProfileName();
	
	// Get the list of supported platform names on this device. Example: Android_ETC2, Android_ASTC
	UFUNCTION(BlueprintPure, Category="Mobile Patching")
	static UE_API TArray<FString> GetSupportedPlatformNames();
};

#undef UE_API
