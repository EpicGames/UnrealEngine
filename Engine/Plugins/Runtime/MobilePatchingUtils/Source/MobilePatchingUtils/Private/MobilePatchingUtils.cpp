// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FMobilePatchingUtilsModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE( FMobilePatchingUtilsModule, MobilePatchingUtils )
